// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class ActionableMessage : ModuleRules
{
	public ActionableMessage(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"UnrealEd"
			}
		);
	}
}
