// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActionableMessageSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActionableMessageSubsystem)

bool operator!=(const FActionableMessage& LHS, const FActionableMessage& RHS)
{
	return !LHS.Message.EqualTo(RHS.Message) || !LHS.ActionMessage.EqualTo(RHS.ActionMessage) || !LHS.Tooltip.EqualTo(RHS.Tooltip)
		|| (!FMemory::Memcmp(&LHS.ActionCallback, &RHS.ActionCallback, sizeof (LHS.ActionCallback)));
}

void UActionableMessageSubsystem::SetActionableMessage(FName InProvider, const FActionableMessage& InActionableMessage)
{
	const TSharedPtr<FActionableMessage>* ValuePtr = ProviderActionableMessageMap.Find(InProvider);

	if (ValuePtr == nullptr)
	{
		ProviderActionableMessageMap.Emplace(InProvider, MakeShared<FActionableMessage>(InActionableMessage));
		++StateID;
	}
	else if	(**ValuePtr != InActionableMessage)
	{
		**ValuePtr = InActionableMessage;
		++StateID;
	}
	
}

void UActionableMessageSubsystem::ClearActionableMessage(FName InProvider)
{
	if (ProviderActionableMessageMap.Remove(InProvider) > 0)
	{
		++StateID;
	}
}
