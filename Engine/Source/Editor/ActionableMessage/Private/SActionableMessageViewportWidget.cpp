// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActionableMessageViewportWidget.h"

#include "ActionableMessageSubsystem.h"

#include "Editor.h"
#include "Engine/World.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "ActionableMessageViewportWidget"

void SActionableMessageEntry::Construct(const FArguments& InArgs)
{
	ActionableMessage = InArgs._ActionableMessage;
	
	ChildSlot
	[
		SNew(SHorizontalBox)
		.ToolTipText_Lambda([this]() { return ActionableMessage.IsValid() ? ActionableMessage->Tooltip : FText::GetEmpty(); })
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
		.AutoWidth()
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("ActionableMessage.Warning"))
			.DesiredSizeOverride(FVector2D(16, 16))
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.FillWidth(1)
		[
			SNew(STextBlock)
			.Text_Lambda([this]() { return ActionableMessage.IsValid() ? ActionableMessage->Message : FText::GetEmpty(); })
			.ColorAndOpacity(FStyleColors::Foreground)
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.AutoWidth()
		[
			SNew(SButton)
			.Visibility_Lambda([this]()
			{
				if (!ActionableMessage.IsValid() || ActionableMessage->ActionMessage.IsEmpty())
				{
					return EVisibility::Collapsed;
				}

				return EVisibility::Visible;
			})
			.ButtonStyle(&FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
			.ContentPadding(FMargin(-4.f, 2.f))
			.OnClicked_Lambda([this]()
			{
				if (ActionableMessage.IsValid() && (ActionableMessage->ActionCallback != nullptr))
				{
					ActionableMessage->ActionCallback();
				}

				return FReply::Handled();
			})
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.f, 0.f, 4.f, 0.f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("ActionableMessage.Update"))
					.DesiredSizeOverride(FVector2D(12, 12))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text_Lambda([this]() { return ActionableMessage.IsValid() ? ActionableMessage->ActionMessage : FText::GetEmpty(); })
					.ColorAndOpacity(FStyleColors::White)
				]
			]
		]
	];
}

void SActionableMessageEntry::SetActionableMessage(TSharedPtr<FActionableMessage> InActionableMessage)
{
	ActionableMessage = InActionableMessage;
}

void SActionableMessageViewportWidget::Construct(const FArguments& InArgs)
{
	constexpr float IconSize = 16.f;
	constexpr float DefaultPadding = 4.f;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		[
			SNew(SButton)
			.ButtonStyle(&FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
			.IsEnabled(true)
			.ToolTipText_Lambda([this]()
			{
				if (!IsExpanded())
				{
					return LOCTEXT("ActionableMessages.CollapsedWarning", "Your project has warnings that need to be resolved. Click for details.");
				}

				return FText();
			})
			.OnClicked_Lambda([this]()
			{
				bExpanded = !bExpanded;
				return FReply::Handled();
			})
			.ContentPadding(FMargin(DefaultPadding))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(FMargin(0.f, 0.f, DefaultPadding, 0.f))
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("ActionableMessage.Warning"))
					.DesiredSizeOverride(FVector2D(IconSize, IconSize))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					SAssignNew(TextBlock, STextBlock)
					.Text_Lambda([this]()
					{
						if (IsExpanded())
						{
							if (ActionableMessages.Num() == 1)
							{
								return LOCTEXT("ActionableMessages.Warning", "Warning");
							}
							
							return FText::Format(LOCTEXT("ActionableMessages.Number", "{0} Warnings"), ActionableMessages.Num());
						}

						return FText::FromString(FString::FromInt(ActionableMessages.Num()));
					})
					.ColorAndOpacity(FStyleColors::Foreground)
					.Font_Lambda([this] ()
					{
						if (IsExpanded())
						{
							return FAppStyle::GetFontStyle(TEXT("BoldFont")); 
						}

						return FAppStyle::GetFontStyle(TEXT("NormalFont"));
					})
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNew(SSpacer)
					.Size_Lambda([this]()
					{
						FVector2d Size(0.f, 0.f);
						
						if (IsExpanded() && ActionableMessageList.IsValid() && TextBlock.IsValid())
						{
							Size.X = ActionableMessageList->GetDesiredSize().X - TextBlock->GetDesiredSize().X - IconSize - 4.f * DefaultPadding - 2.f;
						}
		
						return Size;
					})
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(DefaultPadding, 0.f, 0.f, 0.f))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					SNew(SImage)
					.Image_Lambda([this]() { return IsExpanded() ? FAppStyle::GetBrush("ContentBrowser.SortDown") : FAppStyle::GetBrush("Symbols.LeftArrow"); })
					.Visibility_Lambda([this]() { return ActionableMessages.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; })
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
		+ SVerticalBox::Slot()
		.Padding(0.f, 0.f, 0.f, DefaultPadding)
		.AutoHeight()
		[
			SNew(SSeparator)
			.Thickness(1.f)
			.Visibility_Lambda([this]() {return IsExpanded() ? EVisibility::Visible : EVisibility::Collapsed;})
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(ActionableMessageList, SListView<TSharedPtr<FActionableMessage>>)
			.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("ActionableMessage.ListView"))
			.ListItemsSource(&ActionableMessages)
			.OnGenerateRow(this, &SActionableMessageViewportWidget::OnGenerateRow)
			.Visibility_Lambda([this]()
			{
				return IsExpanded() ? EVisibility::Visible : EVisibility::Collapsed;
			})
		]
	];
}

EVisibility SActionableMessageViewportWidget::GetVisibility()
{
	if (UWorld* World = GEditor->GetEditorWorldContext().World())
	{
		const UActionableMessageSubsystem* ActionableMessageSubsystem = World->GetSubsystem<UActionableMessageSubsystem>();
		
		if (ActionableMessageSubsystem == nullptr)
		{
			return EVisibility::Collapsed;
		}

		TWeakObjectPtr<const UObject> DataSourceID(ActionableMessageSubsystem);  // detect if the data source has changed, probably from loading a new level
		const uint32 SubsystemStateID = ActionableMessageSubsystem->GetStateID();

		if (CachedStateID == SubsystemStateID && CachedDataSourceID == DataSourceID)
		{
			return ActionableMessages.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
		}
		
		const TMap<FName, TSharedPtr<FActionableMessage>>& ActionableMessageSource = ActionableMessageSubsystem->GetActionableMessages();

		ActionableMessages.Empty();
		bForceExpand = false;

		Algo::Transform(ActionableMessageSource, ActionableMessages, [](const TPair<FName, TSharedPtr<FActionableMessage>>& ActionableMessagePair)
		{
			return ActionableMessagePair.Value;
		});

		for (TSharedPtr<FActionableMessage> ActionableMessage : ActionableMessages)
		{
			if (ActionableMessage->bForceExpand)
			{
				bForceExpand = true;
				break;
			}
		}

		CachedStateID = SubsystemStateID;
		CachedDataSourceID = DataSourceID;

		if (ActionableMessageList.IsValid())
		{
			ActionableMessageList->RebuildList();
		}
		
		return ActionableMessages.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

TSharedRef<ITableRow> SActionableMessageViewportWidget::OnGenerateRow(TSharedPtr<FActionableMessage> InActionableMessage, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(InActionableMessage.IsValid());

	return SNew(STableRow<TSharedPtr<FActionableMessage>>, OwnerTable)
	.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ActionableMessage.ListViewRow"))
	.Padding(FMargin(4.f, 0.f, 4.f, 4.f))
	[
		SNew(SActionableMessageEntry)
		.ActionableMessage(InActionableMessage)
	];
}

#undef LOCTEXT_NAMESPACE
