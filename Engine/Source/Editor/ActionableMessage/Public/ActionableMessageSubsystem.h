// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Subsystems/WorldSubsystem.h"

#include "ActionableMessageSubsystem.generated.h"

#define UE_API ACTIONABLEMESSAGE_API

struct FActionableMessage
{
	/**
	 * The message that will be displayed in the widget. 
	 */
	FText Message;

	/**
	 * The text displayed inside the action button, if any. 
	 */
	FText ActionMessage;

	/**
	 * The tooltip of the widget. 
	 */
	FText Tooltip;

	/**
	 * The callback associated with the action button, if any. 
	 */
	TFunction<void()> ActionCallback;

	/**
	 * If true, this message requires the panel to be expanded.
	 */
	bool bForceExpand = false;
};

FORCEINLINE bool operator!=(const FActionableMessage& LHS, const FActionableMessage& RHS);

/**
 * Used to push messages that will be used by a viewport widget to present to the user.
 */
UCLASS(MinimalAPI)
class UActionableMessageSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	/**
	 * Gets all current actionable messages.
	 * @return A map containing current actionable messages.
	 */
	const TMap<FName, TSharedPtr<FActionableMessage>>& GetActionableMessages() const { return ProviderActionableMessageMap; }

	/**
	 * Pushes a unique actionable message.  
	 * @param InProvider The name of the entry.
	 * @param InActionableMessage The content of the actionable message.
	 */
	UE_API void SetActionableMessage(FName InProvider, const FActionableMessage& InActionableMessage);

	/**
	 * Removes an entry from the current actionable messages. 
	 * @param InProvider The name of the entry to remove.
	 */
	UE_API void ClearActionableMessage(FName InProvider);

	/**
	 * Returns the current stateID. Any modification in the current entries will update it. 
	 * @return the current stateID
	 */
	uint32 GetStateID() const { return StateID; }
	
private:
	TMap<FName, TSharedPtr<FActionableMessage>> ProviderActionableMessageMap;
	uint32 StateID = 0;
};

#undef UE_API
