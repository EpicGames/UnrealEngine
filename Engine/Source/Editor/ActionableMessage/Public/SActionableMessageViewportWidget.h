// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActionableMessageSubsystem.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#define UE_API ACTIONABLEMESSAGE_API

class SActionableMessageEntry : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SActionableMessageEntry) {}
	SLATE_ARGUMENT(TSharedPtr<FActionableMessage>, ActionableMessage)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);
	UE_API void SetActionableMessage(TSharedPtr<FActionableMessage> InActionableMessage);

private:
	TSharedPtr<FActionableMessage> ActionableMessage;
	FOnClicked OnClicked;
};

class SActionableMessageViewportWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SActionableMessageViewportWidget) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);
	UE_API EVisibility GetVisibility();

	UE_API TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FActionableMessage> InActionableMessage, const TSharedRef<STableViewBase>& OwnerTable);

private:
	bool IsExpanded() const { return bExpanded || bForceExpand;	}

private:
	TSharedPtr<SListView<TSharedPtr<FActionableMessage>>> ActionableMessageList;
	TSharedPtr<STextBlock> TextBlock;
	TArray<TSharedPtr<FActionableMessage>> ActionableMessages;
	uint32 CachedStateID = 0;
	TWeakObjectPtr<const UObject> CachedDataSourceID;
	bool bExpanded = false;
	bool bForceExpand = false;
};

#undef UE_API
